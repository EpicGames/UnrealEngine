{
	"//1": "Configuration for Apple open source toolchain (clang + OSS headers)",
	"//2": "Used for distributed compilation of non-SDK-dependent Apple platform files",

	"//3": "Version directory within UE_SDKS_ROOT/Host{Platform}/Apple/",
	"//4": "If MainVersion is empty, auto-selects based on XcodeToToolchainVersion mapping",
	"MainVersion": "",

	"//5": "Subdirectory paths relative to the version directory",
	"//6": "Same path on all hosts enables simple UBA Host{Platform} substitution",
	"ClangSubdir": "Clang/bin/clang++",
	"OSSHeadersSubdir": "AppleOSS",

	"//7": "Maps Xcode version to toolchain directory prefix (vN)",
	"//8": "Format: 'FirstXcodeVersion-vN'. Selects highest matching version.",
	"//9": "Xcode    | Apple Clang | LLVM    | Prefix | Auto SDK Directory",
	"//10": "16.0.0+  | 16.0.0      | 17.0.6  | v1     | v1_clang-17.0.6_oss-1350",
	"//11": "16.3.0+  | 17.0.0      | 19.1.4  | v5     | v5_clang-19.1.4_oss-1350_from_source",
	"//12": "26.0.0+  | 17.0.6      | 19.1.6  | v6     | v6_clang-19.1.6_oss-1350_from_source",
	"//13": "26.4.0+  | 21.0.0      | 21.1.6  | v7     | v7_clang-21.1.6_oss-1350_from_source",
	"XcodeToToolchainVersion": [
		"16.0.0-v1",
		"16.3.0-v5",
		"26.0.0-v6",
		"26.4-v7"
	]
}
