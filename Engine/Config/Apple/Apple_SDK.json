{
	"//1": "Xcode versions:",
		"MainVersion": "26.1.1",
		"MinVersion": "15.2.0",
		"MaxVersion": "26.9.0",
	"//2": "!!!",
	"//3": "NOTE: If you update the MaxVersion, double check the AppleVersionToLLVMVersion array below!!!",
	"//4": "!!!",

	"//5": "The versions on Windows are iTunes versions:",
		"MinVersion_Win64": "1100.0.0.0",
		"MaxVersion_Win64": "8999.0",

	"//6": "This is not a version range, but a mapping of Xcode clang versions to LLVM versions, for shared version checks with other clangs",
	"//7": "Version mapping can be found at https://en.wikipedia.org/wiki/Xcode#Toolchain_versions",
	"//8": "The first half of the pair is the first version that is using the second version source LLVM",
	"//9": "For instance, Xcode 16 uses LLVM 17.0.6",
		"AppleVersionToLLVMVersions": [
			"14.0.0-14.0.0",
			"14.0.3-15.0.0",
			"15.0.0-16.0.0",
			"16.0.0-17.0.6",
			"16.3.0-19.1.4",
			"26.0.0-19.1.5",
			"26.4.0-21.1.6"
		]
}
